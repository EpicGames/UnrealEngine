// Copyright Epic Games, Inc. All Rights Reserved.

#include "DefaultSourceCodeAccessor.h"

#define LOCTEXT_NAMESPACE "DefaultSourceCodeAccessor"

bool FDefaultSourceCodeAccessor::CanAccessSourceCode() const
{
	return false;
}

FName FDefaultSourceCodeAccessor::GetFName() const
{
	return FName("None");
}

FText FDefaultSourceCodeAccessor::GetNameText() const
{
	return LOCTEXT("DefaultDisplayName", "None");
}

FText FDefaultSourceCodeAccessor::GetDescriptionText() const
{
	return LOCTEXT("DefaultDisplayDesc", "Do not open source code files.");
}

bool FDefaultSourceCodeAccessor::OpenSolution()
{
	return false;
}

bool FDefaultSourceCodeAccessor::OpenSolutionAtPath(const FString& InSolutionPath)
{
	return false;
}

bool FDefaultSourceCodeAccessor::DoesSolutionExist() const
{
	return false;
}

bool FDefaultSourceCodeAccessor::OpenFileAtLine(const FString& FullPath, int32 LineNumber, int32 ColumnNumber)
{
	return false;
}

bool FDefaultSourceCodeAccessor::OpenSourceFiles(const TArray<FString>& AbsoluteSourcePaths)
{
	return false;
}

bool FDefaultSourceCodeAccessor::AddSourceFiles(const TArray<FString>& AbsoluteSourcePaths, const TArray<FString>& AvailableModules)
{
	return false;
}

bool FDefaultSourceCodeAccessor::SaveAllOpenDocuments() const
{
	return false;
}

void FDefaultSourceCodeAccessor::Tick(const float DeltaTime)
{
}

#undef LOCTEXT_NAMESPACE
