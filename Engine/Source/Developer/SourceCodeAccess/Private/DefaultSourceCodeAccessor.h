// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "ISourceCodeAccessor.h"

class FDefaultSourceCodeAccessor : public ISourceCodeAccessor
{
public:
	/** ISourceCodeAccessor implementation */
	virtual void RefreshAvailability() override { }
	virtual bool CanAccessSourceCode() const override;
	virtual FName GetFName() const override;
	virtual FText GetNameText() const override;
	virtual FText GetDescriptionText() const override;
	virtual bool OpenSolution() override;
	virtual bool OpenSolutionAtPath(const FString& InSolutionPath) override;
	virtual bool DoesSolutionExist() const override;
	virtual bool OpenFileAtLine(const FString& FullPath, int32 LineNumber, int32 ColumnNumber = 0) override;
	virtual bool OpenSourceFiles(const TArray<FString>& AbsoluteSourcePaths) override;
	virtual bool AddSourceFiles(const TArray<FString>& AbsoluteSourcePaths, const TArray<FString>& AvailableModules) override;
	virtual bool SaveAllOpenDocuments() const override;
	virtual void Tick(const float DeltaTime) override;
};
