// Copyright Epic Games, Inc. All Rights Reserved.

#include "ISourceCodeAccessor.h"

#include "Styling/AppStyle.h"

FName ISourceCodeAccessor::GetStyleSet() const
{
	return FAppStyle::GetAppStyleSetName();
}
