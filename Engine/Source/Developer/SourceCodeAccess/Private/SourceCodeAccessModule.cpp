// Copyright Epic Games, Inc. All Rights Reserved.

#include "SourceCodeAccessModule.h"
#include "Modules/ModuleManager.h"
#include "Framework/Application/SlateApplication.h"
#include "Features/IModularFeatures.h"
#include "ISettingsModule.h"
#include "SourceCodeAccessSettings.h"
#include "HAL/LowLevelMemTracker.h"


IMPLEMENT_MODULE( FSourceCodeAccessModule, SourceCodeAccess );

#define LOCTEXT_NAMESPACE "SourceCodeAccessModule"

LLM_DEFINE_TAG(SourceCodeAccess);

static FName SourceCodeAccessorFeatureName(TEXT("SourceCodeAccessor"));


FSourceCodeAccessModule::FSourceCodeAccessModule()
	: CurrentSourceCodeAccessor(nullptr)
{ }


void FSourceCodeAccessModule::StartupModule()
{
	LLM_SCOPE_BYTAG(SourceCodeAccess);

	GetMutableDefault<USourceCodeAccessSettings>()->LoadConfig();

	// Register to check for source control features
	IModularFeatures::Get().OnModularFeatureRegistered().AddRaw(this, &FSourceCodeAccessModule::HandleModularFeatureRegistered);
	IModularFeatures::Get().OnModularFeatureUnregistered().AddRaw(this, &FSourceCodeAccessModule::HandleModularFeatureUnregistered);

	// bind default accessor to editor
	IModularFeatures::Get().RegisterModularFeature(SourceCodeAccessorFeatureName, &DefaultSourceCodeAccessor);

	// Register to display our settings
	ISettingsModule* SettingsModule = FModuleManager::GetModulePtr<ISettingsModule>("Settings");

	if (SettingsModule != nullptr)
	{
		SettingsModule->RegisterSettings("Editor", "General", "Source Code",
			LOCTEXT("TargetSettingsName", "Source Code"),
			LOCTEXT("TargetSettingsDescription", "Control how the editor accesses source code."),
			GetMutableDefault<USourceCodeAccessSettings>()
		);
	}
}


void FSourceCodeAccessModule::ShutdownModule()
{
	// Unregister our settings
	ISettingsModule* SettingsModule = FModuleManager::GetModulePtr<ISettingsModule>("Settings");

	if (SettingsModule != nullptr)
	{
		SettingsModule->UnregisterSettings("Editor", "General", "Source Code");
	}

	// unbind default provider from editor
	IModularFeatures::Get().UnregisterModularFeature(SourceCodeAccessorFeatureName, &DefaultSourceCodeAccessor);

	// we don't care about modular features any more
	IModularFeatures::Get().OnModularFeatureRegistered().RemoveAll(this);
	IModularFeatures::Get().OnModularFeatureUnregistered().RemoveAll(this);
}


bool FSourceCodeAccessModule::CanAccessSourceCode() const
{
	return CurrentSourceCodeAccessor->CanAccessSourceCode();
}

bool FSourceCodeAccessModule::CanCompileSourceCode() const
{
#if PLATFORM_WINDOWS
	// Need to have Visual Studio installed to compile on Windows, regardless of chosen IDE
	return IsSourceCodeAccessorAvailable("VisualStudio2022") || IsSourceCodeAccessorAvailable("VisualStudio2026");
#else
	// Default behavior
	return CanAccessSourceCode();
#endif
}

bool FSourceCodeAccessModule::IsSourceCodeAccessorAvailable(FName Name) const
{
	for (ISourceCodeAccessor* Accessor : IModularFeatures::Get().GetModularFeatureImplementations<ISourceCodeAccessor>(SourceCodeAccessorFeatureName))
	{
		if (Accessor->GetFName() == Name)
		{
			return Accessor->CanAccessSourceCode();
		}
	}
	return false;
}

ISourceCodeAccessor& FSourceCodeAccessModule::GetAccessor() const
{
	return *CurrentSourceCodeAccessor;
}


void FSourceCodeAccessModule::SetAccessor(const FName& InName)
{
	const int32 FeatureCount = IModularFeatures::Get().GetModularFeatureImplementationCount(SourceCodeAccessorFeatureName);
	for(int32 FeatureIndex = 0; FeatureIndex < FeatureCount; FeatureIndex++)
	{
		IModularFeature* Feature = IModularFeatures::Get().GetModularFeatureImplementation(SourceCodeAccessorFeatureName, FeatureIndex);
		check(Feature);

		ISourceCodeAccessor& Accessor = *static_cast<ISourceCodeAccessor*>(Feature);
		if(InName == Accessor.GetFName())
		{
			CurrentSourceCodeAccessor = static_cast<ISourceCodeAccessor*>(Feature);
			if(FSlateApplication::IsInitialized())
			{
				FSlateApplication::Get().SetWidgetReflectorSourceAccessDelegate( FAccessSourceCode::CreateRaw( CurrentSourceCodeAccessor, &ISourceCodeAccessor::OpenFileAtLine ) );
				FSlateApplication::Get().SetWidgetReflectorQuerySourceAccessDelegate( FQueryAccessSourceCode::CreateRaw( CurrentSourceCodeAccessor, &ISourceCodeAccessor::CanAccessSourceCode ) );
			}
			break;
		}
	}
}


FLaunchingCodeAccessor& FSourceCodeAccessModule::OnLaunchingCodeAccessor()
{
	return LaunchingCodeAccessorDelegate;
}

FDoneLaunchingCodeAccessor& FSourceCodeAccessModule::OnDoneLaunchingCodeAccessor()
{
	return DoneLaunchingCodeAccessorDelegate;
}


FOpenFileFailed& FSourceCodeAccessModule::OnOpenFileFailed()
{
	return OpenFileFailedDelegate;
}


void FSourceCodeAccessModule::HandleModularFeatureRegistered(const FName& Type, IModularFeature* ModularFeature)
{
	if(Type == SourceCodeAccessorFeatureName)
	{
		CurrentSourceCodeAccessor = &DefaultSourceCodeAccessor;

		const FString PreferredAccessor = GetDefault<USourceCodeAccessSettings>()->PreferredAccessor;

		const int32 FeatureCount = IModularFeatures::Get().GetModularFeatureImplementationCount(Type);
		for(int32 FeatureIndex = 0; FeatureIndex < FeatureCount; FeatureIndex++)
		{
			IModularFeature* Feature = IModularFeatures::Get().GetModularFeatureImplementation(Type, FeatureIndex);
			check(Feature);

			ISourceCodeAccessor& Accessor = *static_cast<ISourceCodeAccessor*>(Feature);
			if(PreferredAccessor == Accessor.GetFName().ToString())
			{
				CurrentSourceCodeAccessor = static_cast<ISourceCodeAccessor*>(Feature);
				if(FSlateApplication::IsInitialized())
				{
					FSlateApplication::Get().SetWidgetReflectorSourceAccessDelegate( FAccessSourceCode::CreateRaw( CurrentSourceCodeAccessor, &ISourceCodeAccessor::OpenFileAtLine ) );
					FSlateApplication::Get().SetWidgetReflectorQuerySourceAccessDelegate( FQueryAccessSourceCode::CreateRaw( CurrentSourceCodeAccessor, &ISourceCodeAccessor::CanAccessSourceCode ) );
				}
				break;
			}
		}
	}
}


void FSourceCodeAccessModule::HandleModularFeatureUnregistered(const FName& Type, IModularFeature* ModularFeature)
{
	if(Type == SourceCodeAccessorFeatureName && CurrentSourceCodeAccessor == static_cast<ISourceCodeAccessor*>(ModularFeature))
	{
		CurrentSourceCodeAccessor = &DefaultSourceCodeAccessor;
	}
}


#undef LOCTEXT_NAMESPACE
