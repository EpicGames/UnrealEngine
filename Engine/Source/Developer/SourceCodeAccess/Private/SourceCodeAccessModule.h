// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "ISourceCodeAccessModule.h"
#include "DefaultSourceCodeAccessor.h"

/**
 * Implementation of ISourceCodeAccessModule.
 */
class FSourceCodeAccessModule
	: public ISourceCodeAccessModule
{
public:

	/** Default constructor. */
	FSourceCodeAccessModule();

public:

	// IModuleInterface interface

	virtual void StartupModule() override;
	virtual void ShutdownModule() override;

public:

	// ISourceCodeAccessModule interface

	virtual bool CanAccessSourceCode() const override;
	virtual bool CanCompileSourceCode() const override;
	virtual ISourceCodeAccessor& GetAccessor() const override;
	virtual void SetAccessor(const FName& InName) override;
	virtual FLaunchingCodeAccessor& OnLaunchingCodeAccessor() override;
	virtual FDoneLaunchingCodeAccessor& OnDoneLaunchingCodeAccessor() override;
	virtual FOpenFileFailed& OnOpenFileFailed() override;

private:

	/** Handle when one of the modular features we are interested in is registered */
	void HandleModularFeatureRegistered(const FName& Type, IModularFeature* ModularFeature);

	/** Handle when one of the modular features we are interested in is unregistered */
	void HandleModularFeatureUnregistered(const FName& Type, IModularFeature* ModularFeature);

	/** Checks if a source code accessor with the given name is available */
	bool IsSourceCodeAccessorAvailable(FName Name) const;

private:

	/** Event delegate fired when launching code accessor. */
	FLaunchingCodeAccessor LaunchingCodeAccessorDelegate;

	/** Event delegate fired when done launching code accessor.  */
	FDoneLaunchingCodeAccessor DoneLaunchingCodeAccessorDelegate;

	/** Event delegate fired when opening a file has failed. */
	FOpenFileFailed OpenFileFailedDelegate;

	/** The default accessor we will use if we have no IDE. */
	FDefaultSourceCodeAccessor DefaultSourceCodeAccessor;

	/** The current accessor. */
	ISourceCodeAccessor* CurrentSourceCodeAccessor;
};
