// Copyright Epic Games, Inc. All Rights Reserved.

#include "SourceCodeAccessSettings.h"
#include "Misc/ConfigCacheIni.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SourceCodeAccessSettings)

USourceCodeAccessSettings::USourceCodeAccessSettings(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
#if PLATFORM_WINDOWS
	PreferredAccessor = TEXT("VisualStudioSourceCodeAccessor");
#elif PLATFORM_MAC
	PreferredAccessor = TEXT("XCodeSourceCodeAccessor");
#elif PLATFORM_LINUX
	if (!GConfig->GetString(TEXT("/Script/SourceCodeAccess.SourceCodeAccessSettings"), TEXT("PreferredAccessor"), PreferredAccessor, GEditorSettingsIni))
	{
		// If no config is found default to VSCode
		PreferredAccessor = TEXT("VisualStudioCode");
	}

	UE_LOGF(LogHAL, Log, "Linux SourceCodeAccessSettings: %ls", *PreferredAccessor);
#endif
}

