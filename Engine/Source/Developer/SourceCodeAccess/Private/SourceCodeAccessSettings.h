// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "SourceCodeAccessSettings.generated.h"

UCLASS(config=EditorSettings)
class USourceCodeAccessSettings : public UObject
{
	GENERATED_UCLASS_BODY()

	/** The source code editor we prefer to use. */
	UPROPERTY(Config, EditAnywhere, Category="Source Code Editor", meta=(DisplayName="Source Code Editor", ConfigRestartRequired = true))
	FString PreferredAccessor;
};
