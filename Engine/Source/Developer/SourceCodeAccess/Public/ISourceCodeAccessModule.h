// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleInterface.h"

class ISourceCodeAccessor;

/** Event fired when launching code accessor */
DECLARE_MULTICAST_DELEGATE( FLaunchingCodeAccessor );

/**
 * Event fired when done launching code accessor 
 * @param	bSuccess	Whether the launch was successful or not
 */
DECLARE_MULTICAST_DELEGATE_OneParam( FDoneLaunchingCodeAccessor, const bool /*bSuccess*/);

/**
 * Event fired when opening a file has failed
 * @param	InFilename	The filename that failed to open
 */
DECLARE_MULTICAST_DELEGATE_OneParam( FOpenFileFailed, const FString& /*InFilename*/);

/**
 * Module used to access source code
 */
class ISourceCodeAccessModule : public IModuleInterface
{
public:
	/** 
	 * Check to see if source code can be accessed.
	 * @return true if source code can be accessed.
	 */
	virtual bool CanAccessSourceCode() const = 0;

	/** 
	 * Checks if a compiler is available
	 * @return true if source code can be compiled.
	 */
	virtual bool CanCompileSourceCode() const = 0;

	/**
	 * Get the accessor to allow us to view source code
	 * @return the accessor.
	 */
	virtual ISourceCodeAccessor& GetAccessor() const = 0;

	/**
	 * Set the accessor we want to use to view source code
	 * @param	InName	The name of the accessor we want to use
	 */
	virtual void SetAccessor(const FName& InName) = 0;

	/** Gets the Event that is broad casted when attempting to launch visual studio */
	virtual FLaunchingCodeAccessor& OnLaunchingCodeAccessor() = 0;

	/** Gets the Event that is broadcasted when an attempted launch of this code accessor was successful or failed */
	virtual FDoneLaunchingCodeAccessor& OnDoneLaunchingCodeAccessor() = 0;

	/** Gets the Event that is broadcast when an attempt to load a file through Visual Studio failed */
	virtual FOpenFileFailed& OnOpenFileFailed() = 0;
};
