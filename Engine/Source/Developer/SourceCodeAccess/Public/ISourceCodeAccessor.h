// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Features/IModularFeature.h"

/**
 * Interface for viewing/editing source code
 */
class ISourceCodeAccessor : public IModularFeature
{
public:
	virtual ~ISourceCodeAccessor() = default;

	/**
	 * Not necessary to call unless you know you're changing the state of any installed compilers.
	 * If that's the case, you should call this before checking the state the installed state of the
	 * compilers.
	 */
	virtual void RefreshAvailability() = 0;

	/**
	 * Check if we can currently access source code
	 * @return true if source code can be accessed
	 */
	virtual bool CanAccessSourceCode() const = 0;

	/**
	 * Get the name of this source code accessor - used as a unique identifier 
	 * @return the name of this accessor
	 */
	virtual FName GetFName() const = 0;

	/**
	 * Get the name text for this source code accessor
	 * @return the name text of this accessor
	 */
	virtual FText GetNameText() const = 0;

	/**
	 * Get the description text for this source code accessor
	 * @return the description text of this accessor
	 */
	virtual FText GetDescriptionText() const = 0;

	/**
	 * Open the current code solution for editing
	 * @return true if successful
	 */
	virtual bool OpenSolution() = 0;

	/**
	* Open the code solution at a specific path for editing
	* @param InSolutionPath	Path to project directory
	* @return true if successful
	*/
	virtual bool OpenSolutionAtPath(const FString& InSolutionPath) = 0;

	/**
	 * Determine if the source code solution for the given accessor already exists
	 * @return true if solution files exist
	 */
	virtual bool DoesSolutionExist() const = 0;

	/** 
	 * Opens a file in the correct running instance of this code accessor at a line and optionally to a column. 
	 * @param	FullPath		Full path to the file to open
	 * @param	LineNumber		Line number to open the file at
	 * @param	ColumnNumber	Column number to open the file at
	 * @return true if successful
	 */
	virtual bool OpenFileAtLine(const FString& FullPath, int32 LineNumber, int32 ColumnNumber = 0) = 0;
	
	/** 
	 * Opens a group of source files.
	 * @param	AbsoluteSourcePaths		Array of paths to files to open
	 */
	virtual bool OpenSourceFiles(const TArray<FString>& AbsoluteSourcePaths) = 0;

	/**
	 * Add a group of source files to the current solution/project/workspace
	 * @param	AbsoluteSourcePaths		Array of paths to files to open
	 * @param	AvailableModules		Array of known module locations (.Build.cs files) - you can get this from calling FSourceCodeNavigation::GetSourceFileDatabase().GetModuleNames() if in the editor
	 * @return true if the files could be added, false otherwise
	 */
	virtual bool AddSourceFiles(const TArray<FString>& AbsoluteSourcePaths, const TArray<FString>& AvailableModules) = 0;

	/**
	 * Saves all open code documents if they need to be saved.
	 * Will block if there is any read-only files open that need to be saved.
	 * @return true if successful
	 */
	virtual bool SaveAllOpenDocuments() const = 0;

	/**
	 * Tick this source code accessor
	 * @param DeltaTime Delta time (in seconds) since the last call to Tick
	 */
	virtual void Tick(const float DeltaTime) = 0;

	/**
	 * Get the icon that represents opening this source code editor
	 * @return the default source code editor icon
	 */
	virtual FName GetOpenIconName() const
	{
		return FName("MainFrame.OpenSourceCodeEditor"); // Default Open Source Code Editor Icon
	}

	/**
	 * Get the icon that represents refreshing this source code editor
	 * @return the default source code editor icon
	 */
	virtual FName GetRefreshIconName() const
	{
		return FName("MainFrame.RefreshSourceCodeEditor"); // Default Refresh Source Code Editor Icon
	}
	
	/**
	 * Get the style set where custom icons for the editors are defined.
	 * By default uses FAppStyle::GetAppStyleSetName() == "CoreStyle"
	 * @return the default style set for icons
	 */
	SOURCECODEACCESS_API virtual FName GetStyleSet() const;
};
