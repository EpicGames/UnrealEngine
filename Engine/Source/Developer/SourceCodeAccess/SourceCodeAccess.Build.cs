// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class SourceCodeAccess : ModuleRules
{
	public SourceCodeAccess(ReadOnlyTargetRules Target) : base(Target)
	{
		PublicDependencyModuleNames.AddRange(
			new string[] {
				"Core",
				"Settings",
			}
			);

		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"CoreUObject",
				"Slate",
				"SlateCore",
			}
		);
	}
}
